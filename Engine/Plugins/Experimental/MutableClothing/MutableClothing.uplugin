{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mutable Clothing",
	"Description": "Adds Mutable functionality to work with clothing",
	"Category": "Mutable",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"IsExperimentalVersion": true,
	"Modules": [
		{
			"Name": "MutableClothing",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Mutable",
			"Enabled": true
		}
	]
}
