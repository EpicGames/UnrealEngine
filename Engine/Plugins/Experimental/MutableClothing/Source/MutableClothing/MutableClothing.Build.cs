// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MutableClothing : ModuleRules
{
	public MutableClothing(ReadOnlyTargetRules TargetRules) : base(TargetRules)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"ClothingSystemRuntimeCommon",
			});
		
        PublicDependencyModuleNames.AddRange(
	        new string[]
	        {
		        "CustomizableObject",
     		});
    }
}
