// Copyright Epic Games, Inc. All Rights Reserved.


#include "MutableClothingModule.h"

#include "Modules/ModuleManager.h"
#include "ClothingAsset.h"

IMPLEMENT_MODULE(FMutableClothingModule, MutableClothing)

DEFINE_LOG_CATEGORY(LogMutableClothing)

void FMutableClothingModule::StartupModule()
{
}


void FMutableClothingModule::ShutdownModule()
{
}

bool FMutableClothingModule::UpdateClothSimulationLOD(
		int32 InSimulationLODIndex,
		UClothingAssetCommon& InOutClothingAsset,
		TConstArrayView<TArrayView<FMeshToMeshVertData>> InOutAttachedLODsRenderData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MutableClothing_UpdateClothSimulationLOD)

	check(InOutClothingAsset.LodData.IsValidIndex(InSimulationLODIndex));

	if (!InOutAttachedLODsRenderData.Num())
	{
		return false;
	}

	FClothLODDataCommon& ClothingSimData = InOutClothingAsset.LodData[InSimulationLODIndex];
	const int32 NumSimulationVertices = ClothingSimData.PhysicalMeshData.Vertices.Num();
	
	TBitArray<> VertexRemoveMask;
	VertexRemoveMask.Init(true, NumSimulationVertices);

	int32 NumInvalidIndicesFound = 0;
	for (int32 RenderDataIndex = 0; RenderDataIndex < InOutAttachedLODsRenderData.Num(); ++RenderDataIndex)
	{
		for (const FMeshToMeshVertData& VertClothData : InOutAttachedLODsRenderData[RenderDataIndex])
		{
			const uint16* Indices = VertClothData.SourceMeshVertIndices;

			if (Indices[0] < NumSimulationVertices)
			{
				VertexRemoveMask[Indices[0]] = false;
			}

			if (Indices[1] < NumSimulationVertices)
			{
				VertexRemoveMask[Indices[1]] = false;
			}

			if (Indices[2] < NumSimulationVertices)
			{
				VertexRemoveMask[Indices[2]] = false;
			}

#if DO_CHECK
			NumInvalidIndicesFound += 
					static_cast<int32>(Indices[0] >= NumSimulationVertices) +
					static_cast<int32>(Indices[1] >= NumSimulationVertices) +
					static_cast<int32>(Indices[2] >= NumSimulationVertices);
#endif
		}
	}	

	if (NumInvalidIndicesFound > 0)
	{
		UE_LOGF(LogMutableClothing, Error, "Invalid clothing render data indices found.");
	}

	const int32 FirstRemoved = VertexRemoveMask.Find(true);
	if (FirstRemoved == INDEX_NONE)
	{
		// No vertices removed.
		return false;
	}

	TArray<int32> MeshToMeshIndexRemap;
	MeshToMeshIndexRemap.SetNumUninitialized(NumSimulationVertices);
	{
		for (int32 Index = 0; Index < FirstRemoved; ++Index)
		{
			MeshToMeshIndexRemap[Index] = Index;
		}
		
		int32 NewMeshVerticesNum = FirstRemoved;

		for (int32 Index = FirstRemoved; Index < NumSimulationVertices; ++Index)
		{
			MeshToMeshIndexRemap[Index] = !VertexRemoveMask[Index] ? NewMeshVerticesNum++ : INDEX_NONE;
		}

		// Apply index remap to the render data. 
		for (const TArrayView<FMeshToMeshVertData>& VertDataView : InOutAttachedLODsRenderData)
		{
			for (FMeshToMeshVertData& MeshToMeshVertData : VertDataView)
			{
				MeshToMeshVertData.SourceMeshVertIndices[0] = MeshToMeshIndexRemap[MeshToMeshVertData.SourceMeshVertIndices[0]];
				MeshToMeshVertData.SourceMeshVertIndices[1] = MeshToMeshIndexRemap[MeshToMeshVertData.SourceMeshVertIndices[1]];
				MeshToMeshVertData.SourceMeshVertIndices[2] = MeshToMeshIndexRemap[MeshToMeshVertData.SourceMeshVertIndices[2]];
			}

#if DO_CHECK
			int32 NumUnmappedVertices = 0;
			for (FMeshToMeshVertData& MeshToMeshVertData : VertDataView)
			{
				NumUnmappedVertices += static_cast<int32>( 
						(MeshToMeshVertData.SourceMeshVertIndices[0] == INDEX_NONE) |
						(MeshToMeshVertData.SourceMeshVertIndices[1] == INDEX_NONE) |
						(MeshToMeshVertData.SourceMeshVertIndices[2] == INDEX_NONE) );
			}

			check(NumUnmappedVertices == 0);
#endif
		}

		const auto CopyIfUsed = [&VertexRemoveMask](auto& Dst, const auto& Src)
		{
			const int32 SrcNumElems = Src.Num();
			for (int32 Idx = 0, DstNumElems = 0; Idx < SrcNumElems; ++Idx)
			{
				if (!VertexRemoveMask[Idx])
				{
					Dst[DstNumElems++] = Src[Idx];
				}
			}
		};

		FClothLODDataCommon ResultClothingSimData;
	
		ResultClothingSimData.PhysicalMeshData.Vertices.SetNum(NewMeshVerticesNum);
		ResultClothingSimData.PhysicalMeshData.Normals.SetNum(NewMeshVerticesNum);
		ResultClothingSimData.PhysicalMeshData.BoneData.SetNum(NewMeshVerticesNum);
		ResultClothingSimData.PhysicalMeshData.InverseMasses.SetNum(NewMeshVerticesNum);

		CopyIfUsed(ResultClothingSimData.PhysicalMeshData.Vertices,      ClothingSimData.PhysicalMeshData.Vertices);
		CopyIfUsed(ResultClothingSimData.PhysicalMeshData.Normals,       ClothingSimData.PhysicalMeshData.Normals);
		CopyIfUsed(ResultClothingSimData.PhysicalMeshData.BoneData,      ClothingSimData.PhysicalMeshData.BoneData);
		CopyIfUsed(ResultClothingSimData.PhysicalMeshData.InverseMasses, ClothingSimData.PhysicalMeshData.InverseMasses);

		const TMap<uint32, FPointWeightMap>& SourceWeightMaps = ClothingSimData.PhysicalMeshData.WeightMaps;
		TMap<uint32, FPointWeightMap>& DestWeightMaps = ResultClothingSimData.PhysicalMeshData.WeightMaps;

		for (const TPair<uint32, FPointWeightMap>& WeightMap : SourceWeightMaps)
		{
			if (WeightMap.Value.Values.Num() > 0)
			{
				FPointWeightMap& NewWeightMap = ResultClothingSimData.PhysicalMeshData.AddWeightMap(WeightMap.Key);
				NewWeightMap.Values.SetNum(NewMeshVerticesNum);

				CopyIfUsed(NewWeightMap.Values, WeightMap.Value.Values);
			}
		}
	
		const TArray<int32>& IndexMap = MeshToMeshIndexRemap;
		const auto TrimAndRemapTriangles = [&IndexMap](TArray<uint32>& Dst, const TArray<uint32>& Src) -> int32
		{
			check(Src.Num() % 3 == 0);

			const int32 SrcNumElems = Src.Num();

			int32 DstNumElems = 0;
			for (int32 Idx = 0; Idx < SrcNumElems; Idx += 3)
			{
				const int32 Idx0 = IndexMap[Src[Idx + 0]];
				const int32 Idx1 = IndexMap[Src[Idx + 1]];
				const int32 Idx2 = IndexMap[Src[Idx + 2]];

				// triangles are only copied if all vertices are used.
				if (!((Idx0 < 0) | (Idx1 < 0) | (Idx2 < 0)))
				{
					Dst[DstNumElems + 0] = Idx0;
					Dst[DstNumElems + 1] = Idx1;
					Dst[DstNumElems + 2] = Idx2;

					DstNumElems += 3;
				}
			}

			return DstNumElems;
		};

		ResultClothingSimData.PhysicalMeshData.Indices.SetNum(ClothingSimData.PhysicalMeshData.Indices.Num());
		const int32 NumRemainingIndices = TrimAndRemapTriangles(
				ResultClothingSimData.PhysicalMeshData.Indices, 
				ClothingSimData.PhysicalMeshData.Indices);
		ResultClothingSimData.PhysicalMeshData.Indices.SetNum(NumRemainingIndices, EAllowShrinking::No);

		const auto TrimAndRemapVertexSet = [&IndexMap](TSet<int32>& Dst, const TSet<int32>& Src)
		{	
			Dst.Reserve(Src.Num());
			for(const int32 SrcIdx : Src)
			{
				const int32 MappedIdx = IndexMap[SrcIdx];

				if (MappedIdx >= 0)
				{
					Dst.Add(MappedIdx);
				}
			}
		};

		TrimAndRemapVertexSet(
				ResultClothingSimData.PhysicalMeshData.SelfCollisionVertexSet, 
				ClothingSimData.PhysicalMeshData.SelfCollisionVertexSet);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MutableClothing_BuildClothTetherData)

			auto TrimAndRemapTethers = [&IndexMap](FClothTetherData& Dst, const FClothTetherData& Src)
			{
				Dst.Tethers.Reserve(Src.Tethers.Num());
				for (const TArray<TTuple<int32, int32, float>>& SrcTetherCluster : Src.Tethers)
				{
					TArray<TTuple<int32, int32, float>>& DstTetherCluster = Dst.Tethers.Emplace_GetRef();
					DstTetherCluster.Reserve(SrcTetherCluster.Num());
					for (const TTuple<int32, int32, float>& Tether : SrcTetherCluster)
					{
						const int32 Index0 = IndexMap[Tether.Get<0>()];
						const int32 Index1 = IndexMap[Tether.Get<1>()];
						if ((Index0 >= 0) & (Index1 >= 0))
						{
							DstTetherCluster.Emplace(Index0, Index1, Tether.Get<2>());
						}
					}

					if (!DstTetherCluster.Num())
					{
						Dst.Tethers.RemoveAt(Dst.Tethers.Num() - 1, 1, EAllowShrinking::No);
					}
				}
			};

			TrimAndRemapTethers(
					ResultClothingSimData.PhysicalMeshData.GeodesicTethers, 
					ClothingSimData.PhysicalMeshData.GeodesicTethers);

			TrimAndRemapTethers(
					ResultClothingSimData.PhysicalMeshData.EuclideanTethers, 
					ClothingSimData.PhysicalMeshData.EuclideanTethers);
		}

		ClothingSimData.PhysicalMeshData = MoveTemp(ResultClothingSimData.PhysicalMeshData);

		if (ClothingSimData.TransitionUpSkinData.Num())
		{
			ResultClothingSimData.TransitionUpSkinData.SetNum(NewMeshVerticesNum);	
			CopyIfUsed(ResultClothingSimData.TransitionUpSkinData, ClothingSimData.TransitionUpSkinData);

			ClothingSimData.TransitionUpSkinData = MoveTemp(ResultClothingSimData.TransitionUpSkinData);
		}

		if (ClothingSimData.TransitionDownSkinData.Num())
		{
			ResultClothingSimData.TransitionDownSkinData.SetNum(NewMeshVerticesNum);
			CopyIfUsed(ResultClothingSimData.TransitionDownSkinData, ClothingSimData.TransitionDownSkinData);
			
			ClothingSimData.TransitionDownSkinData = MoveTemp(ResultClothingSimData.TransitionDownSkinData);
		}

		auto RemapTransitionMeshToMeshVertData = [](TArray<FMeshToMeshVertData>& InOutVertData, TArray<int32>& IndexMap)
		{
			for (FMeshToMeshVertData& VertData : InOutVertData)
			{
				uint16* Indices = VertData.SourceMeshVertIndices;
				Indices[0] = (uint16)IndexMap[Indices[0]];
				Indices[1] = (uint16)IndexMap[Indices[1]];
				Indices[2] = (uint16)IndexMap[Indices[2]];
			}
		};

		if (InOutClothingAsset.LodData.Num() > InSimulationLODIndex + 1)
		{
			TArray<FMeshToMeshVertData>& NextLodTransitionData = InOutClothingAsset.LodData[InSimulationLODIndex + 1].TransitionUpSkinData;

			if (NextLodTransitionData.Num())
			{
				RemapTransitionMeshToMeshVertData(NextLodTransitionData, MeshToMeshIndexRemap);
			}
		}

		if (InSimulationLODIndex - 1 >= 0)
		{
			TArray<FMeshToMeshVertData>& PrevLodTransitionData = InOutClothingAsset.LodData[InSimulationLODIndex - 1].TransitionDownSkinData;

			if (PrevLodTransitionData.Num())
			{
				RemapTransitionMeshToMeshVertData(PrevLodTransitionData, MeshToMeshIndexRemap);
			}
		}
	}

	return true;
}


void FMutableClothingModule::FixLODTransitionMappings(
		int32 SimulationLODIndex, 
		UClothingAssetCommon& InOutClothingAsset)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MutableClothingModule_FixLODTransitionMappings)

	check(InOutClothingAsset.LodData.IsValidIndex(SimulationLODIndex));

	struct FMeshPhysicsDesc
	{
		const TArray<FVector3f>& Vertices;
		const TArray<FVector3f>& Normals;
		const TArray<uint32>& Indices;
	};

	auto RebindVertex = [](const FMeshPhysicsDesc& Mesh, const FVector3f& InPosition, const FVector3f& InNormal, FMeshToMeshVertData& Out)
	{
		const FVector3f Normal = InNormal;

		// We don't have the mesh tangent, find something plausible.
		FVector3f Tan0, Tan1;
		Normal.FindBestAxisVectors(Tan0, Tan1);
		const FVector3f Tangent = Tan0;
		
		// Some of the math functions take as argument FVector, we'd want to be FVector3f. 
		// This should be changed once support for the single type in the FMath functions is added. 
		const FVector Position = (FVector)InPosition;
		int32 BestBaseTriangleIdx = INDEX_NONE;
		FVector::FReal BestDistanceSq = TNumericLimits<FVector::FReal>::Max();
		
		const int32 NumIndices = Mesh.Indices.Num();
		check(NumIndices % 3 == 0);

		for (int32 I = 0; I < NumIndices; I += 3)
		{
			const FVector& A = (FVector)Mesh.Vertices[Mesh.Indices[I + 0]];
			const FVector& B = (FVector)Mesh.Vertices[Mesh.Indices[I + 1]];
			const FVector& C = (FVector)Mesh.Vertices[Mesh.Indices[I + 2]];

			FVector ClosestTrianglePoint = FMath::ClosestPointOnTriangleToPoint(Position, (FVector)A, (FVector)B, (FVector)C);

			const FVector::FReal CurrentDistSq = (ClosestTrianglePoint - Position).SizeSquared();
			if (CurrentDistSq < BestDistanceSq)
			{
				BestDistanceSq = CurrentDistSq;
				BestBaseTriangleIdx = I;
			}
		}

		check(BestBaseTriangleIdx >= 0);

		auto ComputeBaryCoordsAndDist = [](const FVector3f& A, const FVector3f& B, const FVector3f& C, const FVector3f& P) -> FVector4f
		{
			FPlane4f TrianglePlane(A, B, C);

			const FVector3f PointOnTriPlane = FVector3f::PointPlaneProject(P, TrianglePlane);
			const FVector3f BaryCoords = (FVector3f)FMath::ComputeBaryCentric2D((FVector)PointOnTriPlane, (FVector)A, (FVector)B, (FVector)C);

			return FVector4f(BaryCoords, TrianglePlane.PlaneDot((FVector3f)P));
		};

		const FVector3f& A = Mesh.Vertices[Mesh.Indices[BestBaseTriangleIdx + 0]];
		const FVector3f& B = Mesh.Vertices[Mesh.Indices[BestBaseTriangleIdx + 1]];
		const FVector3f& C = Mesh.Vertices[Mesh.Indices[BestBaseTriangleIdx + 2]];

		Out.PositionBaryCoordsAndDist = ComputeBaryCoordsAndDist(A, B, C, (FVector3f)Position );
		Out.NormalBaryCoordsAndDist = ComputeBaryCoordsAndDist(A, B, C, (FVector3f)Position + Normal );
		Out.TangentBaryCoordsAndDist = ComputeBaryCoordsAndDist(A, B, C, (FVector3f)Position + Tangent );
		Out.SourceMeshVertIndices[0] = (uint16)Mesh.Indices[BestBaseTriangleIdx + 0];
		Out.SourceMeshVertIndices[1] = (uint16)Mesh.Indices[BestBaseTriangleIdx + 1]; 
		Out.SourceMeshVertIndices[2] = (uint16)Mesh.Indices[BestBaseTriangleIdx + 2];
	};

	auto RecreateTransitionData = [&RebindVertex]( 
		const FMeshPhysicsDesc& ToMesh, const FMeshPhysicsDesc& FromMesh, TArray<FMeshToMeshVertData>& InOutTransitionData)
	{
		if (!InOutTransitionData.Num())
		{
			return;
		}

		const int32 TransitionDataNum = InOutTransitionData.Num();
		
		for (int32 I = 0; I < TransitionDataNum; ++I)
		{
			FMeshToMeshVertData& VertData = InOutTransitionData[I];
			uint16* Indices = VertData.SourceMeshVertIndices;

			// If any original indices are missing but the vertex is still alive rebind the vertex.
			// In general, the number of rebinds should be small.

			// Currently, if any index is missing we rebind to the closest triangle but it could be nice to use the remaining indices, 
			// if any, to find the most appropriate triangle to bind to. 
			const bool bNeedsRebind = (Indices[0] == 0xFFFF) | (Indices[1] == 0xFFFF) | (Indices[2] == 0xFFFF);

			if (bNeedsRebind)
			{
				RebindVertex(ToMesh, FromMesh.Vertices[I], FromMesh.Normals[I], VertData);
			}
		}
	};

	const int32 NumLODs = InOutClothingAsset.LodData.Num();

	const FMeshPhysicsDesc CurrentPhysicsMesh {
			InOutClothingAsset.LodData[SimulationLODIndex].PhysicalMeshData.Vertices,
			InOutClothingAsset.LodData[SimulationLODIndex].PhysicalMeshData.Normals,
			InOutClothingAsset.LodData[SimulationLODIndex].PhysicalMeshData.Indices };

	if (SimulationLODIndex + 1 < NumLODs && 
		InOutClothingAsset.LodData[SimulationLODIndex].TransitionDownSkinData.Num())
	{
		const FMeshPhysicsDesc TransitionUpTarget {  
				InOutClothingAsset.LodData[SimulationLODIndex + 1].PhysicalMeshData.Vertices,
				InOutClothingAsset.LodData[SimulationLODIndex + 1].PhysicalMeshData.Normals,
				InOutClothingAsset.LodData[SimulationLODIndex + 1].PhysicalMeshData.Indices };
		
		if (TransitionUpTarget.Vertices.Num())
		{
			RecreateTransitionData(
					TransitionUpTarget, 
					CurrentPhysicsMesh, 
					InOutClothingAsset.LodData[SimulationLODIndex].TransitionDownSkinData);
		}
		else
		{
			InOutClothingAsset.LodData[SimulationLODIndex].TransitionDownSkinData.Empty();
		}
	}
	
	if (SimulationLODIndex - 1 >= 0 &&
		InOutClothingAsset.LodData[SimulationLODIndex].TransitionUpSkinData.Num())
	{	
		FMeshPhysicsDesc TransitionDownTarget {  
				InOutClothingAsset.LodData[SimulationLODIndex - 1].PhysicalMeshData.Vertices,
				InOutClothingAsset.LodData[SimulationLODIndex - 1].PhysicalMeshData.Normals,
				InOutClothingAsset.LodData[SimulationLODIndex - 1].PhysicalMeshData.Indices };

		if (TransitionDownTarget.Vertices.Num())
		{
			RecreateTransitionData(
					TransitionDownTarget, 
					CurrentPhysicsMesh, 
					InOutClothingAsset.LodData[SimulationLODIndex].TransitionUpSkinData);
		}
		else
		{
			InOutClothingAsset.LodData[SimulationLODIndex].TransitionUpSkinData.Empty();
		}
	}
}
