// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/Plugins/IMutableClothingModule.h"
#include "Logging/LogMacros.h"

#define UE_API MUTABLECLOTHING_API


MUTABLECLOTHING_API DECLARE_LOG_CATEGORY_EXTERN(LogMutableClothing, Log, All);

class FMutableClothingModule : public IMutableClothingModule
{
public:
	// IModuleInterface interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	// IMutableClothingModule interface
	UE_API virtual bool UpdateClothSimulationLOD(
			int32 InSimulationLODIndex,	
			UClothingAssetCommon& InOutClothingAsset,
			TConstArrayView<TArrayView<FMeshToMeshVertData>> InOutAttachedLODsRenderData) override;

	UE_API virtual void FixLODTransitionMappings(
			int32 InSimulationLODIndex, 
			UClothingAssetCommon& InOutClothingAsset) override;
};

#undef UE_API
